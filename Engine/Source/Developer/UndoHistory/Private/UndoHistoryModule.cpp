// Copyright Epic Games, Inc. All Rights Reserved.

#include "UndoHistoryModule.h"

#define LOCTEXT_NAMESPACE "FUndoHistoryModule"

void FUndoHistoryModule::StartupModule()
{}

void FUndoHistoryModule::ShutdownModule()
{}

bool FUndoHistoryModule::SupportsDynamicReloading()
{
	return true;
}


IMPLEMENT_MODULE(FUndoHistoryModule, UndoHistory);


#undef LOCTEXT_NAMESPACE
