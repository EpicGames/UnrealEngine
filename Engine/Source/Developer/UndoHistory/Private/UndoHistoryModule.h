// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IUndoHistoryModule.h"

/**
* Implements the UndoHistory module.
*/
class FUndoHistoryModule : public IUndoHistoryModule
{
public:

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
	//~ End IModuleInterface Interface
};
