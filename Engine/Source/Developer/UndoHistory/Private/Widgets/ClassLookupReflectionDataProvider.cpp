// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassLookupReflectionDataProvider.h"

#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"

namespace UE::UndoHistory
{
	bool FClassLookupReflectionDataProvider::HasClassDisplayName(const FSoftClassPath& ClassPath) const
	{
		return LookUpClass(ClassPath).IsSet();
	}

	TOptional<FString> FClassLookupReflectionDataProvider::GetClassDisplayName(const FSoftClassPath& ClassPath) const
	{
		const TOptional<TNonNullPtr<UClass>> Class = LookUpClass(ClassPath);
		return Class ? (*Class)->GetName() : TOptional<FString>{};
	}

	TOptional<FPropertyReflectionData> FClassLookupReflectionDataProvider::GetPropertyReflectionData(const FSoftClassPath& ClassPath, FName PropertyName) const
	{
		if (const TOptional<TNonNullPtr<FProperty>> Property = LookUpProperty(ClassPath, PropertyName))
		{
			FPropertyReflectionData Result{}; // Value init so if new members are added they have defined values
			Result.PropertyType = GetPropertyType(**Property);
			(*Property)->GetCPPMacroType(Result.CppMacroType);
			Result.TypeName = (*Property)->GetClass()->GetName();
			Result.PropertyFlags = (*Property)->PropertyFlags;
			return Result;
		}
		return {};
	}

	TOptional<TNonNullPtr<UClass>> FClassLookupReflectionDataProvider::LookUpClass(const FSoftClassPath& ClassPath) const
	{
		UClass* Class = LoadObject<UClass>(nullptr, *ClassPath.ToString());
		return LIKELY(Class) ? TOptional<TNonNullPtr<UClass>>{ Class } : TOptional<TNonNullPtr<UClass>>{};
	}

	TOptional<TNonNullPtr<FProperty>> FClassLookupReflectionDataProvider::LookUpProperty(const FSoftClassPath& ClassPath, FName PropertyName) const
	{
		if (const TOptional<TNonNullPtr<UClass>> Class = LookUpClass(ClassPath))
		{
			FProperty* Property = (*Class)->FindPropertyByName(PropertyName);
			return LIKELY(Property) ? TOptional<TNonNullPtr<FProperty>>{ Property } : TOptional<TNonNullPtr<FProperty>>{};
		}
		return {};
	}

	TSharedRef<IReflectionDataProvider> CreateDefaultReflectionProvider()
	{
		return MakeShared<FClassLookupReflectionDataProvider>();
	}

	EPropertyType FClassLookupReflectionDataProvider::GetPropertyType(const FProperty& Property) const
	{
		if (CastField<FObjectPropertyBase>(&Property))
		{
			return EPropertyType::ObjectProperty;
		}
		if (CastField<FStructProperty>(&Property))
		{
			return EPropertyType::StructProperty;
		}
		if (CastField<FEnumProperty>(&Property))
		{
			return EPropertyType::EnumProperty;
		}
		if (CastField<FArrayProperty>(&Property))
		{
			return EPropertyType::StructProperty;
		}
		return EPropertyType::Other;
	}
}


