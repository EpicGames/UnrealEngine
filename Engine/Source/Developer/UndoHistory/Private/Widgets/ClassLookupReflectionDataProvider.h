// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/NonNullPointer.h"
#include "Widgets/IReflectionDataProvider.h"

namespace UE::UndoHistory
{
	/** Just uses LoadObject to look up class data. Works on editor builds but not on most programs, like MultiUserServer. */
	class FClassLookupReflectionDataProvider : public IReflectionDataProvider
	{
	public:

		//~ Begin IReflectionDataProvider Interface
		virtual bool HasClassDisplayName(const FSoftClassPath& ClassPath) const override;
		virtual TOptional<FString> GetClassDisplayName(const FSoftClassPath& ClassPath) const override;
		virtual bool SupportsGetPropertyReflectionData() const override { return true; }
		virtual TOptional<FPropertyReflectionData> GetPropertyReflectionData(const FSoftClassPath& ClassPath, FName PropertyName) const override;
		//~ End IReflectionDataProvider Interface

	private:

		TOptional<TNonNullPtr<UClass>> LookUpClass(const FSoftClassPath& ClassPath) const;
		TOptional<TNonNullPtr<FProperty>> LookUpProperty(const FSoftClassPath& ClassPath, FName PropertyName) const;
		EPropertyType GetPropertyType(const FProperty& Property) const;
	};
}


