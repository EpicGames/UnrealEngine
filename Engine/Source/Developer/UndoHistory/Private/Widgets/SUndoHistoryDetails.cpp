// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SUndoHistoryDetails.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UndoHistoryUtils.h"
#include "Misc/AsciiSet.h"
#include "Misc/PackageName.h"
#include "Misc/TransactionObjectEvent.h"
#include "Algo/Transform.h"

#define LOCTEXT_NAMESPACE "SUndoHistoryDetails"

// Static Transaction Details UI FNames
namespace TransactionDetailsUI
{
	static const FName NameLabel(TEXT("Name"));
	static const FName TypeLabel(TEXT("Type"));
	static const FName ChangeFlagsLabel(TEXT("ChangeFlags"));
 
	TSharedRef<SWidget> CreateIconSpacer()
	{
		return SNew(SBox)
			.WidthOverride(16)
			.HeightOverride(16);
	}

	TSharedRef<SWidget> CreateRenameIcon(const FName OriginalName)
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
			.ToolTipText(FText::Format(LOCTEXT("RenameChangeToolTip", "Renamed.\nOriginal Name: {0}"), FText::FromName(OriginalName)))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}

	TSharedRef<SWidget> CreateOuterChangeIcon(const FName OriginalOuter)
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowUp"))
			.ToolTipText(FText::Format(LOCTEXT("OuterChangeToolTip", "Has an outer change.\nOriginal Outer: {0}"), FText::FromName(OriginalOuter)))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}

	TSharedRef<SWidget> CreatePendingKillIcon()
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
			.ToolTipText(LOCTEXT("PendingKillToolTip", "Has a pending kill change."))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}

	TSharedRef<SWidget> CreateNonPropertyChangeIcon()
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			.ToolTipText(LOCTEXT("NonPropertyChangeToolTip", "Has a non-property change."))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}

	TSharedRef<SWidget> CreateAnnotationIcon()
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Info"))
			.ToolTipText(LOCTEXT("AnnotationToolTip", "Has an annotation."))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}
};

/**
 * Implements a row widget for the UndoDetails tree view.
 */
class SUndoHistoryDetailsRow
	: public SMultiColumnTableRow<TSharedPtr<int32> >
{

public:

	SLATE_BEGIN_ARGS(SUndoHistoryDetailsRow)
	: _TransactionEvent(nullptr)
	{ }
		SLATE_ARGUMENT(TSharedPtr<FTransactionObjectEvent>, TransactionEvent)
		SLATE_ARGUMENT(FString, Name)
		SLATE_ARGUMENT(FString, Type)
		SLATE_ATTRIBUTE(FText, FilterText)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		TransactionEvent = InArgs._TransactionEvent;
		Name = InArgs._Name;
		Type = InArgs._Type;
		FilterText = InArgs._FilterText;

		SMultiColumnTableRow<TSharedPtr<int32> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TransactionDetailsUI::NameLabel)
		{
			return
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
							.IndentAmount(8)
							.ShouldDrawWires(true)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromString(Name))
							.HighlightText(FilterText)
					];
		}
		else if (ColumnName == TransactionDetailsUI::TypeLabel)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Type));
		}
		else if (ColumnName == TransactionDetailsUI::ChangeFlagsLabel)
		{
			if (!TransactionEvent.IsValid())
			{
				return SNullWidget::NullWidget;
			}

			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f, 0.0f))
				[
					CreateFlagBox()
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:
	TSharedRef<SHorizontalBox> CreateFlagBox()
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		HorizontalBox->AddSlot()
			.Padding(FMargin(2.0f, 0.0f))
			[
				TransactionEvent->HasNameChange() ? TransactionDetailsUI::CreateRenameIcon(TransactionEvent->GetOriginalObjectName()) : TransactionDetailsUI::CreateIconSpacer()
			];

		HorizontalBox->AddSlot()
			.Padding(FMargin(2.0f, 0.0f))
			[
				TransactionEvent->HasOuterChange() ? TransactionDetailsUI::CreateOuterChangeIcon(TransactionEvent->GetOriginalObjectOuterPathName()) : TransactionDetailsUI::CreateIconSpacer()
			];

		HorizontalBox->AddSlot()
			.Padding(FMargin(2.0f, 0.0f))
			[
				TransactionEvent->HasPendingKillChange() ? TransactionDetailsUI::CreatePendingKillIcon() : TransactionDetailsUI::CreateIconSpacer()
			];

		HorizontalBox->AddSlot()
			.Padding(FMargin(2.0f, 0.0f))
			[
				TransactionEvent->HasNonPropertyChanges() ? TransactionDetailsUI::CreateNonPropertyChangeIcon() : TransactionDetailsUI::CreateIconSpacer()
			];

		HorizontalBox->AddSlot()
			.Padding(FMargin(2.0f, 0.0f))
			[
				TransactionEvent->GetAnnotation().IsValid() ? TransactionDetailsUI::CreateAnnotationIcon() : TransactionDetailsUI::CreateIconSpacer()
			];

		return HorizontalBox;
	}

private:

	TSharedPtr<FTransactionObjectEvent> TransactionEvent;
	FString Name;
	FString Type;
	TAttribute<FText> FilterText;

};

void SUndoHistoryDetails::Construct(const FArguments& InArgs, TSharedRef<UE::UndoHistory::IReflectionDataProvider> InReflectionData)
{
	ReflectionData = MoveTemp(InReflectionData);

	auto FilterDelegate = FTreeItemTextFilter::FItemToStringArray::CreateSP(this, &SUndoHistoryDetails::PopulateSearchStrings);
	SearchBoxFilter = MakeShared<FTreeItemTextFilter>(FilterDelegate);
	SearchBoxFilter->OnChanged().AddSP(this, &SUndoHistoryDetails::FullRefresh);
	
	bNeedsRefresh = true;
	bNeedsExpansion = false;
	
	const auto OnGetChildren = [](FUndoDetailsTreeNodePtr InNode, TArray<FUndoDetailsTreeNodePtr> & OutChildren)
	{
		OutChildren = InNode->Children;
	};
	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4.0f, 1.0f))
				[
					SNew(SVerticalBox)
						.Clipping(EWidgetClipping::ClipToBounds)
						.Visibility(this, &SUndoHistoryDetails::HandleDetailsVisibility)

					+ SVerticalBox::Slot()
						.Padding(FMargin(0.0f, 4.0f))
						.AutoHeight()
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
								.Padding(FMargin(0.0f, 1.0f))
								[
									SNew(SGridPanel)

									+ SGridPanel::Slot(0, 0)
									[
										SNew(STextBlock)
											.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
											.Text(LOCTEXT("TransactionActionLabel", "Action:"))
									]

									+ SGridPanel::Slot(1, 0)
										.Padding(16.0f, 0.0f, 8.0f, 0.0f)
										[
											SNew(STextBlock)
											.Text(this, &SUndoHistoryDetails::HandleTransactionName)
										]

									+ SGridPanel::Slot(0, 1)
										.Padding(0.0f, 4.0f, 0.0f, 0.0f)
										[
											SNew(STextBlock)
												.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
												.Text(LOCTEXT("TransactionIdLabel", "Transaction Id:"))
										]

									+ SGridPanel::Slot(1, 1)
										.Padding(16.0f, 4.0f, 8.0f, 0.0f)
										[
											SNew(SHyperlink)
												.ToolTipText(NSLOCTEXT("SUndoHistoryDetails", "ClickToCopy", "Click to copy ID."))
												.Text(this, &SUndoHistoryDetails::HandleTransactionId)
												.OnNavigate(this, &SUndoHistoryDetails::HandleTransactionIdNavigate)
										]
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(FMargin(0.0f, 5.0f))
						[
							SNew(SSeparator)
								.Orientation(EOrientation::Orient_Horizontal)
						]

					+ SVerticalBox::Slot()
						.Padding(FMargin(0.0f, 3.0f))
						.AutoHeight()
						[
							SAssignNew(FilterTextBoxWidget, SSearchBox)
								.HintText(LOCTEXT("FilterSearch", "Search..."))
								.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search for a modified object or property."))
								.OnTextChanged(this, &SUndoHistoryDetails::OnFilterTextChanged)
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(FMargin(0.0f, 4.0f))
						[
							SAssignNew(ChangedObjectsTreeView, STreeView<FUndoDetailsTreeNodePtr>)
								.TreeItemsSource(&FilteredChangedObjects)
								.OnGenerateRow(this, &SUndoHistoryDetails::HandleGenerateRow)
								.OnGetChildren_Static(OnGetChildren)
								.SelectionMode(ESelectionMode::Single)
								.HeaderRow
								(
									SNew(SHeaderRow)
									+ SHeaderRow::Column(TransactionDetailsUI::NameLabel)
									.FillWidth(40.0f)
									.DefaultLabel(LOCTEXT("NameColumnHeaderName", "Modified objects and properties"))
									+ SHeaderRow::Column(TransactionDetailsUI::TypeLabel)
									.FillWidth(20.0f)
									.ShouldGenerateWidget(SupportsTypeRow())
									.DefaultLabel(LOCTEXT("TypeColumnHeaderName", "Type"))
									+ SHeaderRow::Column(TransactionDetailsUI::ChangeFlagsLabel)
									.FillWidth(15.0f)
									.DefaultLabel(LOCTEXT("ChangeFlagsHeaderName", "Change Flags"))
								)
						]
				]
		];
}

void SUndoHistoryDetails::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) 
{
	if (bNeedsRefresh)
	{
		Populate();
	}
}

void SUndoHistoryDetails::SetSelectedTransaction(const FTransactionDiff& InTransactionDiff)
{
	ChangedObjects.Reset();

	TransactionName = FText::FromString(InTransactionDiff.TransactionTitle);
	TransactionId = FText::FromString(InTransactionDiff.TransactionId.ToString());

	for (const auto& ObjectIt : InTransactionDiff.DiffMap)
	{
		const FSoftClassPath ClassPath = ObjectIt.Value->GetOriginalObjectClassPathName().ToString();
		if (ReflectionData->HasClassDisplayName(ClassPath))
		{
			ChangedObjects.Emplace(CreateTreeNode(ObjectIt.Key.ToString(), ClassPath, ObjectIt.Value));
		}
	}

	// Sort the list so that outer objects appear before inner objects
	ChangedObjects.StableSort([](const FUndoDetailsTreeNodePtr& A, const FUndoDetailsTreeNodePtr& B)
	{
		return A->PathDepth < B->PathDepth;
	});

	Populate();
}

void SUndoHistoryDetails::Reset()
{
	ChangedObjects.Reset();
	ChangedObjectsTreeView->RequestTreeRefresh();
}

SUndoHistoryDetails::FUndoDetailsTreeNodePtr SUndoHistoryDetails::CreateTreeNode(const FString& InObjectPathName, const FSoftClassPath& InObjectClass, const TSharedPtr<FTransactionObjectEvent>& InEvent) const
{
	const TOptional<FString> ClassName = ReflectionData->GetClassDisplayName(InObjectClass);
	check(ClassName);
	
	int32 ObjectPathDepth = 0;
	{
		constexpr FAsciiSet PathSepratorCharacters(".:");
		for (const TCHAR* It = *InObjectPathName; *It != 0; It = FAsciiSet::FindFirstOrEnd(It + 1, PathSepratorCharacters))
		{
			++ObjectPathDepth;
		}
	}

	FUndoDetailsTreeNodePtr ObjectNode = MakeShared<FUndoDetailsTreeNode>(FPackageName::ObjectPathToObjectName(InObjectPathName), *ClassName, FText::AsCultureInvariant(InObjectPathName), InEvent, ObjectPathDepth);

	TArray<FUndoHistoryUtils::FBasicPropertyInfo> BasicPropertyInfo = FUndoHistoryUtils::GetChangedPropertiesInfo(*ReflectionData, InObjectClass, InEvent->GetChangedProperties());

	for (const auto& It : BasicPropertyInfo)
	{
		ObjectNode->Children.Emplace(MakeShared<FUndoDetailsTreeNode>(It.PropertyName, It.PropertyType, CreateToolTipText(It.PropertyFlags)));
	}

	return ObjectNode;
}

FText SUndoHistoryDetails::CreateToolTipText(EPropertyFlags InFlags) const
{
	if (InFlags == CPF_None)
	{
		return {};
	}
	
	FString ToolTipString = TEXT("Flags:\n");
	ToolTipString += FString::Join(ParsePropertyFlags(InFlags), TEXT("\n"));

	return FText::FromString(MoveTemp(ToolTipString));
}

bool SUndoHistoryDetails::SupportsTypeRow() const
{
	return ReflectionData->SupportsGetPropertyReflectionData();
}

void SUndoHistoryDetails::OnFilterTextChanged(const FText& InFilterText)
{
	SearchBoxFilter->SetRawFilterText(InFilterText);
	FilterTextBoxWidget->SetError(SearchBoxFilter->GetFilterErrorText());
;}

void SUndoHistoryDetails::FullRefresh()
{
	bNeedsExpansion = !SearchBoxFilter->GetRawFilterText().IsEmpty();
	bNeedsRefresh = true;
}

void SUndoHistoryDetails::PopulateSearchStrings(const FString& InItemName, TArray< FString >& OutSearchStrings) const
{
	OutSearchStrings.Add(InItemName);
}

void SUndoHistoryDetails::Populate()
{
	FilteredChangedObjects.Reset();

	for (const auto& ChangedObject : ChangedObjects)
	{
		FUndoDetailsTreeNodePtr ObjectNode = MakeShared<FUndoDetailsTreeNode>(*ChangedObject);

 		ObjectNode->Children.Reset();

		for (const auto& ChangedProperty : ChangedObject->Children)
		{
			if (SearchBoxFilter->PassesFilter(ChangedProperty->Name))
			{
				ObjectNode->Children.Add(ChangedProperty);
			}
		}

		if (ObjectNode->Children.Num() != 0 || SearchBoxFilter->PassesFilter(ObjectNode->Name))
		{
			FilteredChangedObjects.Add(ObjectNode);
			ChangedObjectsTreeView->SetItemExpansion(ObjectNode, bNeedsExpansion);
		}
	}

	ChangedObjectsTreeView->RequestTreeRefresh();

	bNeedsRefresh = false;
}

TSharedRef<ITableRow> SUndoHistoryDetails::HandleGenerateRow(FUndoDetailsTreeNodePtr InNode, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SUndoHistoryDetailsRow, OwnerTable)
		.Name(InNode->Name)
		.Type(InNode->Type)
		.FilterText(this, &SUndoHistoryDetails::HandleGetFilterHighlightText)
		.ToolTipText(InNode->ToolTip)
		.TransactionEvent(InNode->TransactionEvent);
}

FText SUndoHistoryDetails::HandleGetFilterHighlightText() const
{
	return SearchBoxFilter->GetRawFilterText();
}

EVisibility SUndoHistoryDetails::HandleDetailsVisibility() const
{
	return ChangedObjects.Num() > 0 ? EVisibility::Visible : EVisibility::Hidden;
}

FText SUndoHistoryDetails::HandleTransactionName() const
{
	return TransactionName;
}

FText SUndoHistoryDetails::HandleTransactionId() const
{
	return TransactionId;
}

void SUndoHistoryDetails::HandleTransactionIdNavigate()
{
	FPlatformApplicationMisc::ClipboardCopy(*TransactionId.ToString());
}

#undef LOCTEXT_NAMESPACE