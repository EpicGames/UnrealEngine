// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Widgets/IReflectionDataProvider.h"

class FUndoHistoryUtils
{
public:

	struct FBasicPropertyInfo
	{
		FString PropertyName;
		FString PropertyType;
		EPropertyFlags PropertyFlags = CPF_None;

		FBasicPropertyInfo(FString InPropertyName, FString InPropertyType = FString(), EPropertyFlags InPropertyFlags = CPF_None)
			: PropertyName(MoveTemp(InPropertyName))
			, PropertyType(MoveTemp(InPropertyType))
			, PropertyFlags(InPropertyFlags)
		{}
	};

	static TArray<FBasicPropertyInfo> GetChangedPropertiesInfo(const UE::UndoHistory::IReflectionDataProvider& ReflectionData, const FSoftClassPath& InObjectClass, const TArray<FName>& InChangedProperties)
	{
		if (ReflectionData.SupportsGetPropertyReflectionData())
		{
			return GetChangedPropertiesInfoUsingReflectionInterface(ReflectionData, InObjectClass, InChangedProperties);
		}
		else
		{
			return GetChangedPropertiesWithoutReflectionInfo(InChangedProperties);
		}
	}

private:
	
	static TArray<FBasicPropertyInfo> GetChangedPropertiesInfoUsingReflectionInterface(const UE::UndoHistory::IReflectionDataProvider& ReflectionData, const FSoftClassPath& InObjectClass, const TArray<FName>& InChangedProperties)
	{
		using namespace UE::UndoHistory;
		TArray<FBasicPropertyInfo> Properties;
		
		for (const FName& PropertyName : InChangedProperties)
		{
			if (TOptional<FPropertyReflectionData> PropertyData = ReflectionData.GetPropertyReflectionData(InObjectClass, PropertyName))
			{
				FString ClassName;
				if (PropertyData->PropertyType == UE::UndoHistory::EPropertyType::ObjectProperty
					|| PropertyData->PropertyType == UE::UndoHistory::EPropertyType::StructProperty
					|| PropertyData->PropertyType == UE::UndoHistory::EPropertyType::EnumProperty)
				{
					ClassName = PropertyData->CppMacroType;
				}
				else if (PropertyData->PropertyType == UE::UndoHistory::EPropertyType::ArrayProperty)
				{
					ClassName = PropertyData->CppMacroType;
					ClassName = FString::Printf(TEXT("TArray<%s>"), *ClassName);
				}
				else
				{
					ClassName = PropertyData->TypeName;
					ClassName.RemoveFromEnd("Property");
				}

				Properties.Emplace(PropertyName.ToString(), MoveTemp(ClassName), PropertyData->PropertyFlags);
			}
			else
			{
				Properties.Emplace(PropertyName.ToString());
			}
		}
		
		return Properties;
	}

	static TArray<FBasicPropertyInfo> GetChangedPropertiesWithoutReflectionInfo(const TArray<FName>& InChangedProperties)
	{
		TArray<FBasicPropertyInfo> Properties;
		for (const FName& PropertyName : InChangedProperties)
		{
			Properties.Emplace(PropertyName.ToString());
		}
		return Properties;
	}
};