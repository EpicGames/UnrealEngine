// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * Interface for the UndoHistory module.
 */
class IUndoHistoryModule
	: public IModuleInterface
{
public:

	/**
	 * Gets a reference to the UndoHistory module instance.
	 *
	 * @todo gmp: better implementation using dependency injection.
	 * @return A reference to the module.
	 */
	static IUndoHistoryModule& Get( )
	{
		return FModuleManager::LoadModuleChecked<IUndoHistoryModule>("UndoHistory");
	}

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~IUndoHistoryModule( ) = default;
};
