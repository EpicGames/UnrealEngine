// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"

namespace UE::UndoHistory
{
	enum class EPropertyType : uint8
	{
		ObjectProperty,
		StructProperty,
		EnumProperty,
		ArrayProperty,
		Other
	};

	struct FPropertyReflectionData
	{
		/** The result of FProperty::GetCPPMacroType (the part that is returned by ref, e.g. "FName"). */
		FString CppMacroType;
		/** The result of FProperty::GetClass()::GetName() */
		FString TypeName;
		EPropertyFlags PropertyFlags;
		EPropertyType PropertyType;
	};

	
	/**
	 * Provides reflection data to SUndoHistoryDetails.
	 * 
	 * On editor builds reflection data is fully available however on non-editor builds there is no straightforward way of obtaining it.
	 * For example, Concert implements this interface to return data where available.
	 */
	class IReflectionDataProvider
	{
	public:

		virtual bool HasClassDisplayName(const FSoftClassPath& ClassPath) const = 0;
		/** @return What UClass::GetName() would return, if available */
		virtual TOptional<FString> GetClassDisplayName(const FSoftClassPath& ClassPath) const = 0;

		virtual bool SupportsGetPropertyReflectionData() const = 0;
		/** @return Various misc data required for UI, e.g. for the type column */		
		virtual TOptional<FPropertyReflectionData> GetPropertyReflectionData(const FSoftClassPath& ClassPath, FName PropertyName) const = 0;
		
		virtual ~IReflectionDataProvider() = default;
	};

	/** Gets the default provider that just calls LoadObject. Works on editor builds. Will not work in most programs, like MultiUserServer. */
	UNDOHISTORY_API TSharedRef<IReflectionDataProvider> CreateDefaultReflectionProvider();
}
