// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IReflectionDataProvider.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Misc/TextFilter.h"

#define UE_API UNDOHISTORY_API

class SUndoHistoryDetails : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS(SUndoHistoryDetails) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedRef<UE::UndoHistory::IReflectionDataProvider> InReflectionData = UE::UndoHistory::CreateDefaultReflectionProvider());

	/**
	 * Set the transaction to be displayed in the details panel.
	 */
	UE_API void SetSelectedTransaction(const struct FTransactionDiff& InTransactionDiff);

	/**
	 * Clear the details panel.
	 */
	UE_API void Reset();


	/** SWidget interface */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	struct FUndoDetailsTreeNode;
	using FUndoDetailsTreeNodePtr = TSharedPtr<FUndoDetailsTreeNode>;
	using FTreeItemTextFilter = TTextFilter<const FString&>;

	/** Tree node representing a changed object and its changed properties as children. */
	struct FUndoDetailsTreeNode
	{
		FString Name;
		FString Type;
		FText ToolTip;
		TSharedPtr<FTransactionObjectEvent> TransactionEvent;
		TArray<FUndoDetailsTreeNodePtr> Children;
		int32 PathDepth = 0;

		FUndoDetailsTreeNode(FString InName, FString InType, FText InToolTip, TSharedPtr<FTransactionObjectEvent> InTransactionEvent = nullptr, int32 InPathDepth = 0)
			: Name(MoveTemp(InName))
			, Type(MoveTemp(InType))
			, ToolTip(MoveTemp(InToolTip))
			, TransactionEvent(MoveTemp(InTransactionEvent))
			, PathDepth(InPathDepth)
		{}
	};

private:

	/** Create a changed object node. */
	UE_API FUndoDetailsTreeNodePtr CreateTreeNode(const FString& InObjectPathName, const FSoftClassPath& InObjectClass, const TSharedPtr<FTransactionObjectEvent>& InEvent) const;

	/** Create a tooltip text for a property. */
	UE_API FText CreateToolTipText(EPropertyFlags InFlags) const;

	/** Whether the type row will be generated */
	UE_API bool SupportsTypeRow() const;

	/** Callback to handle a change in the filter box. */
	UE_API void OnFilterTextChanged(const FText& InFilterText);

	/** Refresh the details tree view. */
	UE_API void FullRefresh();

	/** Populate the search strings for the filter. */
	UE_API void PopulateSearchStrings(const FString&, TArray<FString>& OutSearchStrings) const;

	/** Populate the details tree. */
	UE_API void Populate();

	/** Callback for generating a UndoHistoryDetailsRow */
	UE_API TSharedRef<ITableRow> HandleGenerateRow(FUndoDetailsTreeNodePtr InNode, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Callback for getting the filter highlight text. */
	UE_API FText HandleGetFilterHighlightText() const;

	/** Callback for getting the details visibility. */
	UE_API EVisibility HandleDetailsVisibility() const;

	/** Callback for getting the transaction name. */
	UE_API FText HandleTransactionName() const;

	/* Callback for getting the transaction id. */
	UE_API FText HandleTransactionId() const;

	/** Callback for handling a click on the transaction id. */
	UE_API void HandleTransactionIdNavigate();

private:

	/** Gives us information about property data (gives programs the opportunity to inject custom handler since not all reflection data is available in programs). */
	TSharedPtr<UE::UndoHistory::IReflectionDataProvider> ReflectionData;
	
	/** Holds the ChangedObjects TreeView. */
	TSharedPtr<STreeView<FUndoDetailsTreeNodePtr> > ChangedObjectsTreeView;

	/** Holds the ChangedObjects to be used as an ItemSource to the TreeView. */
	TArray<FUndoDetailsTreeNodePtr> ChangedObjects;

	/** Holds the ChangedObjects to be displayed. */
	TArray<FUndoDetailsTreeNodePtr> FilteredChangedObjects;

	/** Holds the search box. */
	TSharedPtr<class SSearchBox> FilterTextBoxWidget;

	/** Holds the TransactionName. */
	FText TransactionName;

	/** Holds the TransactionId. */
	FText TransactionId;

	/** The TextFilter attached to the SearchBox widget of the UndoHistoryDetails panel. */
	TSharedPtr<FTreeItemTextFilter> SearchBoxFilter;

	/** If the details tree needs to be refreshed. */
	bool bNeedsRefresh;

	/** If the tree items need to be expanded (ie. When the filter text changes). */
	bool bNeedsExpansion;
};

#undef UE_API
